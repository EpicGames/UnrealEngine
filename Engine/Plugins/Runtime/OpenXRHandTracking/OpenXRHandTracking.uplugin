{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenXRHandTracking",
	"Description": "OpenXR Hand Tracking provides XR_EXT_hand_tracking support.",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "OpenXRHandTracking",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OpenXRHandTrackingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		}
	]
}
