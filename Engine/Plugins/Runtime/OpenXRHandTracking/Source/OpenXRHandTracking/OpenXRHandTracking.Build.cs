// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class OpenXRHandTracking: ModuleRules
    {
        public OpenXRHandTracking(ReadOnlyTargetRules Target) 
				: base(Target)
        {
			PublicDependencyModuleNames.Add("InputDevice");

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
					"HeadMountedDisplay",
					"XRBase",
                    "InputCore",
					"LiveLinkAnimationCore",
					"LiveLinkInterface",
					"OpenXRHMD",
					"OpenXRInput",
					"Slate",
					"SlateCore",
					"ApplicationCore"
				}
				);

            AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenXR");

            if (Target.bBuildEditor == true)
            {
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
				PrivateDependencyModuleNames.Add("InputEditor");
            }
        }
    }
}
