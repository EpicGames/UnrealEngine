// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenXRHandTrackingSourceFactory.h"
#include "IOpenXRHandTrackingModule.h"

#include "ILiveLinkClient.h"

#define LOCTEXT_NAMESPACE "OpenXRHandTracking"

FText ULiveLinkOpenXRHandTrackingSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("HandTrackingLiveLinkSourceName", "Windows Mixed Reality Hand Tracking Source");
}

FText ULiveLinkOpenXRHandTrackingSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("HandTrackingLiveLinkSourceTooltip", "Windows Mixed Reality Hand Tracking Key Points Source");
}

ULiveLinkOpenXRHandTrackingSourceFactory::EMenuType ULiveLinkOpenXRHandTrackingSourceFactory::GetMenuType() const
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		if (!IOpenXRHandTrackingModule::Get().IsLiveLinkSourceValid() || !LiveLinkClient.HasSourceBeenAdded(IOpenXRHandTrackingModule::Get().GetLiveLinkSource()))
		{
			return EMenuType::MenuEntry;
		}
	}
	return EMenuType::Disabled;
}

TSharedPtr<ILiveLinkSource> ULiveLinkOpenXRHandTrackingSourceFactory::CreateSource(const FString& ConnectionString) const
{
	return IOpenXRHandTrackingModule::Get().GetLiveLinkSource();
}

#undef LOCTEXT_NAMESPACE
