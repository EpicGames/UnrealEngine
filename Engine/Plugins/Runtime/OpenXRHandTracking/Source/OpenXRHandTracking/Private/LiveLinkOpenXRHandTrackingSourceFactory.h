// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceFactory.h"
#include "LiveLinkOpenXRHandTrackingSourceFactory.generated.h"

UCLASS()
class ULiveLinkOpenXRHandTrackingSourceFactory : public ULiveLinkSourceFactory
{
public:

	GENERATED_BODY()

	virtual FText GetSourceDisplayName() const;
	virtual FText GetSourceTooltip() const;

	virtual EMenuType GetMenuType() const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;

	TSharedPtr<class SLiveLinkOpenXRHandTrackingSourceEditor> ActiveSourceEditor;
};