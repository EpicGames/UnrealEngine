// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHandTracking.h"
#include "IOpenXRHandTrackingModule.h"

#include "IXRTrackingSystem.h"
#include "Engine/Engine.h"
#include "OpenXRCore.h"
#include "Framework/Application/SlateApplication.h"
//#include "DrawDebugHelpers.h"
#include "ILiveLinkClient.h"
#include "IOpenXRHMD.h"
#include "IOpenXRHMDModule.h"
#include "OpenXRHandTrackingSettings.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "OpenXRHandTracking"

// These enum's must match.
static_assert(EHandKeypoint::Palm == static_cast<EHandKeypoint>(XR_HAND_JOINT_PALM_EXT), "EHandKeypoint enum does not match XrHandJointEXT.");
static_assert(EHandKeypoint::Wrist == static_cast<EHandKeypoint>(XR_HAND_JOINT_WRIST_EXT), "EHandKeypoint enum does not match XrHandJointEXT.");
static_assert(EHandKeypoint::ThumbMetacarpal == static_cast<EHandKeypoint>(XR_HAND_JOINT_THUMB_METACARPAL_EXT), "EHandKeypoint enum does not match XrHandJointEXT.");
static_assert(EHandKeypoint::IndexTip == static_cast<EHandKeypoint>(XR_HAND_JOINT_INDEX_TIP_EXT), "EHandKeypoint enum does not match XrHandJointEXT.");
static_assert(EHandKeypoint::LittleTip == static_cast<EHandKeypoint>(XR_HAND_JOINT_LITTLE_TIP_EXT), "EHandKeypoint enum does not match XrHandJointEXT.");

class FOpenXRHandTrackingModule :
	public IOpenXRHandTrackingModule
{
public:
	FOpenXRHandTrackingModule()
		: InputDevice(nullptr)
		, bLiveLinkSourceRegistered(false)
	{}

	virtual void StartupModule() override
	{
		IOpenXRHandTrackingModule::StartupModule();

		// HACK: Generic Application might not be instantiated at this point so we create the input device with a
		// dummy message handler. When the Generic Application creates the input device it passes a valid message
		// handler to it which is further on used for all the controller events. This hack fixes issues caused by
		// using a custom input device before the Generic Application has instantiated it. Eg. within BeginPlay()
		//
		// This also fixes the warnings that pop up on the custom input keys when the blueprint loads. Those
		// warnings are caused because Unreal loads the bluerints before the input device has been instantiated
		// and has added its keys, thus leading Unreal to believe that those keys don't exist. This hack causes
		// an earlier instantiation of the input device, and consequently, the custom keys.
		TSharedPtr<FGenericApplicationMessageHandler> DummyMessageHandler(new FGenericApplicationMessageHandler());
		CreateInputDevice(DummyMessageHandler.ToSharedRef());
	}

	virtual void ShutdownModule() override
	{
		IOpenXRHandTrackingModule::ShutdownModule();
	}

	virtual TSharedPtr<class IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override
	{
		if (!InputDevice.IsValid())
		{
			TSharedPtr<FOpenXRHandTracking> HandTrackingInputDevice(new FOpenXRHandTracking(InMessageHandler));
			InputDevice = HandTrackingInputDevice;

			return InputDevice;
		}
		else
		{
			InputDevice.Get()->SetMessageHandler(InMessageHandler);
			return InputDevice;
		}
	}

	virtual TSharedPtr<IInputDevice> GetInputDevice() override
	{
		if (!InputDevice.IsValid())
		{
			CreateInputDevice(FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler());
		}
		return InputDevice;
	}

	virtual TSharedPtr<ILiveLinkSource> GetLiveLinkSource() override
	{
		if (!InputDevice.IsValid())
		{
			CreateInputDevice(FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler());
		}
		return InputDevice;
	}

	virtual bool IsLiveLinkSourceValid() const override
	{
		return InputDevice.IsValid();
	}

	virtual void AddLiveLinkSource() override
	{
		if (bLiveLinkSourceRegistered)
		{
			return;
		}
		// Auto register with LiveLink
		ensureMsgf(FModuleManager::Get().LoadModule("LiveLink"), TEXT("OpenXRHandTracking depends on the LiveLink module."));
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
			LiveLinkClient->AddSource(GetLiveLinkSource());
			bLiveLinkSourceRegistered = true;
		}
	}

	virtual void RemoveLiveLinkSource() override
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
			LiveLinkClient->RemoveSource(GetLiveLinkSource());
		}
		bLiveLinkSourceRegistered = false;
	}

private:
	TSharedPtr<FOpenXRHandTracking> InputDevice;
	bool bLiveLinkSourceRegistered;
};

IMPLEMENT_MODULE(FOpenXRHandTrackingModule, OpenXRHandTracking);


FLiveLinkSubjectName FOpenXRHandTracking::LiveLinkLeftHandTrackingSubjectName(TEXT("LeftHand"));
FLiveLinkSubjectName FOpenXRHandTracking::LiveLinkRightHandTrackingSubjectName(TEXT("RightHand"));


FOpenXRHandTracking::FOpenXRHandTracking(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	: MessageHandler(InMessageHandler)
	, DeviceIndex(0)
{
	// Register modular feature manually
	IModularFeatures::Get().RegisterModularFeature(IMotionController::GetModularFeatureName(), static_cast<IMotionController*>(this));
	IModularFeatures::Get().RegisterModularFeature(IHandTracker::GetModularFeatureName(), static_cast<IHandTracker*>(this));
	IModularFeatures::Get().RegisterModularFeature(IOpenXRExtensionPlugin::GetModularFeatureName(), static_cast<IOpenXRExtensionPlugin*>(this));
	AddKeys();

	// We're implicitly requiring that the OpenXRPlugin has been loaded and
	// initialized at this point.
	if (!IOpenXRHMDModule::Get().IsAvailable())
	{
		UE_LOGF(LogOpenXRHandTracking, Error, "Error - OpenXRHMDPlugin isn't available");
	}
}

FOpenXRHandTracking::~FOpenXRHandTracking()
{
	// Unregister modular feature manually
	IModularFeatures::Get().UnregisterModularFeature(IMotionController::GetModularFeatureName(), static_cast<IMotionController*>(this));
	IModularFeatures::Get().UnregisterModularFeature(IHandTracker::GetModularFeatureName(), static_cast<IHandTracker*>(this));
	IModularFeatures::Get().UnregisterModularFeature(IOpenXRExtensionPlugin::GetModularFeatureName(), static_cast<IOpenXRExtensionPlugin*>(this));
}

bool FOpenXRHandTracking::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	TArray<XrApiLayerProperties> Properties;
	EnumerateOpenXRApiLayers(Properties);

	// Some API layers can crash the loader when enabled, if they're present we shouldn't enable the extension
	for (const XrApiLayerProperties& Layer : Properties)
	{
		if (FCStringAnsi::Strstr(Layer.layerName, "XR_APILAYER_VIVE_hand_tracking") &&
			Layer.layerVersion <= 1)
		{
			return false;
		}
	}

	OutExtensions.Add(XR_EXT_HAND_TRACKING_EXTENSION_NAME);
	return true;
}

const void* FOpenXRHandTracking::OnGetSystem(XrInstance InInstance, const void* InNext)
{
	// Store extension open xr calls to member function pointers for convenient use.
	XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrCreateHandTrackerEXT", (PFN_xrVoidFunction*)&xrCreateHandTrackerEXT));
	XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrDestroyHandTrackerEXT", (PFN_xrVoidFunction*)&xrDestroyHandTrackerEXT));
	XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrLocateHandJointsEXT", (PFN_xrVoidFunction*)&xrLocateHandJointsEXT));

	return InNext;
}

const void* FOpenXRHandTracking::OnCreateSession(XrInstance InInstance, XrSystemId InSystem, const void* InNext)
{
	// Need to wait until the EHandKeypoint enum has been loaded, so we do this here rather than in the constructor which runs too early.
	BuildMotionSourceToKeypointMap();

	XrSystemHandTrackingPropertiesEXT HandTrackingSystemProperties{
	XR_TYPE_SYSTEM_HAND_TRACKING_PROPERTIES_EXT };
	XrSystemProperties systemProperties{ XR_TYPE_SYSTEM_PROPERTIES,
										&HandTrackingSystemProperties };
	XR_ENSURE(xrGetSystemProperties(InInstance, InSystem, &systemProperties));

	bHandTrackingAvailable = HandTrackingSystemProperties.supportsHandTracking == XR_TRUE;

	return InNext;
}

const void* FOpenXRHandTracking::OnBeginSession(XrSession InSession, const void* InNext)
{
	if (bHandTrackingAvailable)
	{
		static FName SystemName(TEXT("OpenXR"));
		if (GEngine->XRSystem.IsValid() && (GEngine->XRSystem->GetSystemName() == SystemName))
		{
			XRTrackingSystem = GEngine->XRSystem.Get();
		}

		// Create a hand tracker for left hand that tracks default set of hand joints.
		FHandState& LeftHandState = GetLeftHandState();
		CreateHandTracker(InSession, XR_HAND_LEFT_EXT, LeftHandState);

		// Create a hand tracker for right  hand that tracks default set of hand joints.
		FHandState& RightHandState = GetRightHandState();
		CreateHandTracker(InSession, XR_HAND_RIGHT_EXT, RightHandState);
	}

	return InNext;
}

void FOpenXRHandTracking::UpdateDeviceLocations(XrSession InSession, XrTime DisplayTime, XrSpace TrackingSpace)
{
	if (!bHandTrackingAvailable)
	{
		return;
	}

	XrHandJointsLocateInfoEXT LocateInfo{ XR_TYPE_HAND_JOINTS_LOCATE_INFO_EXT };
	LocateInfo.baseSpace = TrackingSpace;
	LocateInfo.time = DisplayTime;

	const float WorldToMetersScale = XRTrackingSystem->GetWorldToMetersScale();

	for (int i = 0; i < 2; ++i)
	{
		FHandState& HandState = HandStates[i];

		XR_ENSURE(xrLocateHandJointsEXT(HandState.HandTracker, &LocateInfo, &HandState.Locations));

		HandState.bTracked = HandState.Locations.isActive == XR_TRUE;
		HandState.HasReceivedJointPoses |= HandState.bTracked;
		if (HandState.bTracked) {

			static_assert(XR_HAND_JOINT_PALM_EXT == 0 && XR_HAND_JOINT_LITTLE_TIP_EXT == XR_HAND_JOINT_COUNT_EXT - 1, "XrHandJointEXT enum is not as assumed for the following loop!");
			for (int j = 0; j < XR_HAND_JOINT_COUNT_EXT; ++j)
			{
				const XrHandJointLocationEXT& JoinLocation = HandState.JointLocations[j];
				const XrPosef& Pose = JoinLocation.pose;
				const FTransform Transform = ToFTransform(Pose, WorldToMetersScale);
				HandState.KeypointTransforms[j] = Transform;
				HandState.Radii[j] = JoinLocation.radius * WorldToMetersScale;
				// velocities would go here
			}
		}
	}
}

FOpenXRHandTracking::FHandState::FHandState()
{
	Velocities.jointCount = XR_HAND_JOINT_COUNT_EXT;
	Velocities.jointVelocities = JointVelocities;

	Locations.next = &Velocities;
	Locations.jointCount = XR_HAND_JOINT_COUNT_EXT;
	Locations.jointLocations = JointLocations;
}

bool FOpenXRHandTracking::FHandState::GetTransform(EHandKeypoint Keypoint, FTransform& OutTransform) const
{
	check((int32)Keypoint < EHandKeypointCount);
	OutTransform = KeypointTransforms[(uint32)Keypoint];
	
	return HasReceivedJointPoses;
}

const FTransform& FOpenXRHandTracking::FHandState::GetTransform(EHandKeypoint Keypoint) const
{
	check((int32)Keypoint < EHandKeypointCount);
	return KeypointTransforms[(uint32)Keypoint];
}

void FOpenXRHandTracking::BuildMotionSourceToKeypointMap()
{
	if (!MotionSourceToKeypointMap.IsEmpty())
	{
		return;
	}

	bool bUseMoreSpecificMotionSourceNames = false;
	UOpenXRHandTrackingSettings* Settings = GetMutableDefault<UOpenXRHandTrackingSettings>();
	ensure(Settings);
	if (Settings)
	{
		bUseMoreSpecificMotionSourceNames = Settings->bUseMoreSpecificMotionSourceNames;
		bSupportLegacyControllerMotionSources = Settings->bSupportLegacyControllerMotionSources;
	}

	// There is a motionsource that corresponds to each hand keypoint of the form [Left|Right][Keypoint].
	// Build a map so we can quickly translate from motion source FName to hand bone EHandKeypoint value.
	// We also have the option of using more specific motion sources of the form HandTracking[Left|Right][Keypoint]
	// this is useful if one wishes to use hand tracking and controllers simultaneously.
	// We also may support more generic legacy motion sources, by default we do support this.
	const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/HeadMountedDisplay.EHandKeypoint"), EFindObjectFlags::ExactClass);
	check(EnumPtr != nullptr);

	check(IsInGameThread());
	const FString Left(bUseMoreSpecificMotionSourceNames ? TEXT("HandTrackingLeft") : TEXT("Left"));
	const FString Right(bUseMoreSpecificMotionSourceNames ? TEXT("HandTrackingRight") : TEXT("Right"));
	for (int64 E = 0; E < EHandKeypointCount; ++E)
	{
		const EHandKeypoint EnumValue = static_cast<EHandKeypoint>(E);
		const FString EnumName = EnumPtr->GetNameStringByValue(E);

		const FName LeftName(Left + EnumName);
		const FName RightName(Right + EnumName);
		MotionSourceToKeypointMap.Add(LeftName, MotionSourceInfo(EnumValue, true));
		MotionSourceToKeypointMap.Add(RightName, MotionSourceInfo(EnumValue, false));
	}

	if (bSupportLegacyControllerMotionSources)
	{
		MotionSourceToKeypointMap.Add(FName("Left"), MotionSourceInfo(EHandKeypoint::Palm, true));
		MotionSourceToKeypointMap.Add(FName("Right"), MotionSourceInfo(EHandKeypoint::Palm, false));
	}
}

bool FOpenXRHandTracking::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const
{
	if (!bHandTrackingAvailable)
	{
		return false;
	}
	
	// Hand tracking currently does not support late update.  Current hand tracking systems have latency that make it pointless.
	if (!IsInGameThread())
	{
		return false;
	}

	bool bTracked = false;
	if (ControllerIndex == DeviceIndex)
	{
		FTransform ControllerTransform = FTransform::Identity;

		const MotionSourceInfo* KeyPointInfoPtr = MotionSourceToKeypointMap.Find(MotionSource);
		if (KeyPointInfoPtr)
		{
			const EHandKeypoint KeyPoint = KeyPointInfoPtr->Key;
			const bool bIsLeft = KeyPointInfoPtr->Value;
			const FHandState& HandState = bIsLeft ? GetLeftHandState() : GetRightHandState();
			if (HandState.HasReceivedJointPoses)
			{
				ControllerTransform = HandState.GetTransform(KeyPoint);
				bTracked = HandState.bTracked;
			}
		}

		OutPosition = ControllerTransform.GetLocation();
		OutOrientation = ControllerTransform.GetRotation().Rotator();
	}

	return bTracked;
}

ETrackingStatus FOpenXRHandTracking::GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const
{
	if (!bHandTrackingAvailable)
	{
		return ETrackingStatus::NotTracked;
	}

	const MotionSourceInfo* KeyPointInfoPtr = MotionSourceToKeypointMap.Find(MotionSource);
	if (KeyPointInfoPtr)
	{
		const bool bIsLeft = KeyPointInfoPtr->Value;
		const FOpenXRHandTracking::FHandState& HandState = bIsLeft ? GetLeftHandState() : GetRightHandState();
		return HandState.bTracked ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;
	}

	return ETrackingStatus::NotTracked;
}

FName FOpenXRHandTracking::GetMotionControllerDeviceTypeName() const
{
	const static FName DefaultName(TEXT("OpenXRHandTracking"));
	return DefaultName;
}

void FOpenXRHandTracking::EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const
{
	check(IsInGameThread());

	bool bUseMoreSpecificMotionSourceNames = false;
	UOpenXRHandTrackingSettings* Settings = GetMutableDefault<UOpenXRHandTrackingSettings>();
	ensure(Settings);
	if (Settings)
	{
		bUseMoreSpecificMotionSourceNames = Settings->bUseMoreSpecificMotionSourceNames;
	}

	SourcesOut.Reserve(SourcesOut.Num() + (EHandKeypointCount * 2));

	const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/HeadMountedDisplay.EHandKeypoint"), EFindObjectFlags::ExactClass);
	check(EnumPtr != nullptr);
	const FString Left(bUseMoreSpecificMotionSourceNames ? TEXT("HandTrackingLeft") : TEXT("Left"));
	const FString Right(bUseMoreSpecificMotionSourceNames ? TEXT("HandTrackingRight") : TEXT("Right"));
	for (int32 Keypoint = 0; Keypoint < EHandKeypointCount; Keypoint++)
	{
		static int32 EnumNameLength = FString(TEXT("EHandKeypoint::")).Len();

		const FString EnumString = EnumPtr->GetNameByValue(Keypoint).ToString();
		const TCHAR* EnumChars = *EnumString;
		const TCHAR* EnumValue = EnumChars + EnumNameLength;
		FString StringLeft(Left);
		StringLeft.AppendChars(EnumValue, EnumString.Len() - EnumNameLength);
		FString StringRight(Right);
		StringRight.AppendChars(EnumValue, EnumString.Len() - EnumNameLength);
		FName SourceL(*(StringLeft));
		FName SourceR(*(StringRight));
		SourcesOut.Add(SourceL);
		SourcesOut.Add(SourceR);
	}
}

void FOpenXRHandTracking::Tick(float DeltaTime)
{
	UpdateLiveLink();
}

void FOpenXRHandTracking::SendControllerEvents()
{
}

void FOpenXRHandTracking::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

bool FOpenXRHandTracking::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return false;
}

bool FOpenXRHandTracking::IsGamepadAttached() const
{
	return false;
}

FOpenXRHandTracking::FHandState& FOpenXRHandTracking::GetLeftHandState()
{
	return HandStates[0];
}

FOpenXRHandTracking::FHandState& FOpenXRHandTracking::GetRightHandState()
{
	return HandStates[1];
}

const FOpenXRHandTracking::FHandState& FOpenXRHandTracking::GetLeftHandState() const
{
	return HandStates[0];
}

const FOpenXRHandTracking::FHandState& FOpenXRHandTracking::GetRightHandState() const
{
	return HandStates[1];
}

bool FOpenXRHandTracking::IsHandTrackingSupportedByDevice() const
{
	return bHandTrackingAvailable;
}

FName FOpenXRHandTracking::GetHandTrackerDeviceTypeName() const
{
	return GetMotionControllerDeviceTypeName();
}

bool FOpenXRHandTracking::IsHandTrackingStateValid() const
{
	return bHandTrackingAvailable;
}

bool FOpenXRHandTracking::GetKeypointState(EControllerHand Hand, EHandKeypoint Keypoint, FTransform& OutTransform, float& OutRadius) const
{
	if (!bHandTrackingAvailable)
	{
		return false;
	}

	bool gotTransform = false;

	// NOTE: currently there is no openxr input simulation implementation.  Maybe we will do that soon though?  Leaving this for reference for now.
	//#if WITH_INPUT_SIMULATION
	//	if (auto* InputSim = UOpenXRInputSimulationEngineSubsystem::GetInputSimulationIfEnabled())
	//	{
	//		gotTransform = InputSim->GetHandJointTransform(Hand, Keypoint, OutTransform);
	//		OutRadius = HandState.Radii[(uint32)Keypoint];
	//	}
	//	else
	//#endif
	{
		const FOpenXRHandTracking::FHandState& HandState = (Hand == EControllerHand::Left) ? GetLeftHandState() : GetRightHandState();
		gotTransform = HandState.GetTransform(Keypoint, OutTransform);
		OutRadius = HandState.Radii[(uint32)Keypoint];
	}
	if (gotTransform)
	{
		// Convert to UE world space
		const FTransform& TrackingToWoldTransform = XRTrackingSystem->GetTrackingToWorldTransform();
		OutTransform *= TrackingToWoldTransform;
	}
	return gotTransform;
}

bool FOpenXRHandTracking::GetAllKeypointStates(EControllerHand Hand, TArray<FVector>& OutPositions, TArray<FQuat>& OutRotations, TArray<float>& OutRadii, bool& OutIsTracked) const
{
	if (!bHandTrackingAvailable)
	{
		return false;
	}

	if (Hand != EControllerHand::Left && Hand != EControllerHand::Right)
	{
		return false;
	}

	const FOpenXRHandTracking::FHandState& HandState = (Hand == EControllerHand::Left) ? GetLeftHandState() : GetRightHandState();

	if (!HandState.HasReceivedJointPoses)
	{
		return false;
	}

	OutIsTracked = HandState.bTracked;

	// If we ever received poses we return the last data we received, even if not currently tracking to avoid pops back to 0,0,0 and the like.
	OutPositions.Empty(EHandKeypointCount);
	OutRotations.Empty(EHandKeypointCount);
	const FTransform& TrackingToWoldTransform = XRTrackingSystem->GetTrackingToWorldTransform();
	for (int i = 0; i < EHandKeypointCount; ++i)
	{
		FTransform KeypointWorldTransform = HandState.KeypointTransforms[i] * TrackingToWoldTransform;
		OutPositions.Add(KeypointWorldTransform.GetLocation());
		OutRotations.Add(KeypointWorldTransform.GetRotation());
	}

	OutRadii.Empty(EHandKeypointCount);
	for (int i = 0; i < EHandKeypointCount; ++i)
	{
		OutRadii.Add(HandState.Radii[i]);
	}

	return true;
}

void FOpenXRHandTracking::AddKeys()
{
}

void FOpenXRHandTracking::CreateHandTracker(XrSession InSession, XrHandEXT HandEXT, FHandState& HandState)
{
	XrHandTrackerCreateInfoEXT CreateInfo{ XR_TYPE_HAND_TRACKER_CREATE_INFO_EXT };
	CreateInfo.hand = HandEXT;
	CreateInfo.handJointSet = XR_HAND_JOINT_SET_DEFAULT_EXT;

	if (XRTrackingSystem && XRTrackingSystem->GetIOpenXRHMD())
	{
		TArray<IOpenXRExtensionPlugin*> ExtensionPlugins;
		ExtensionPlugins = XRTrackingSystem->GetIOpenXRHMD()->GetExtensionPlugins();
		for (IOpenXRExtensionPlugin* Plugin : ExtensionPlugins)
		{
			CreateInfo.next = Plugin->OnCreateHandTracker(CreateInfo.next);
		}
	}
	
	XR_ENSURE(xrCreateHandTrackerEXT(InSession, &CreateInfo, &HandState.HandTracker));
}

#undef LOCTEXT_NAMESPACE
