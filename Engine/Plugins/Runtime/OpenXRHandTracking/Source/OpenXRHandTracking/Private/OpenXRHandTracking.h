// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadMountedDisplayTypes.h"
#include "XRMotionControllerBase.h"
#include "ILiveLinkSource.h"
#include "IInputDevice.h"
#include "IHandTracker.h"

#include "IOpenXRExtensionPlugin.h"

enum class ETrackingStatus : uint8;
struct FMotionControllerSource;

/**
  * OpenXR HandTracking
  */
class FOpenXRHandTracking :
	public IOpenXRExtensionPlugin,
	public IInputDevice,
	public FXRMotionControllerBase,
	public IHandTracker,
	public ILiveLinkSource
{
public:
	struct FHandState : public FNoncopyable
	{
		FHandState();

		XrHandTrackerEXT HandTracker{};
		XrHandJointLocationEXT JointLocations[XR_HAND_JOINT_COUNT_EXT];
		XrHandJointVelocityEXT JointVelocities[XR_HAND_JOINT_COUNT_EXT];
		XrHandJointVelocitiesEXT Velocities{ XR_TYPE_HAND_JOINT_VELOCITIES_EXT };
		XrHandJointLocationsEXT Locations{ XR_TYPE_HAND_JOINT_LOCATIONS_EXT };

		// Transforms are cached in Unreal Tracking Space
		FTransform KeypointTransforms[EHandKeypointCount];
		float Radii[EHandKeypointCount];
		bool HasReceivedJointPoses = false; // true if this hand has ever been tracked, and therefore its poses are valid.
		bool bTracked = false; // true if tracked right now.

		bool GetTransform(EHandKeypoint KeyPoint, FTransform& OutTransform) const;
		const FTransform& GetTransform(EHandKeypoint KeyPoint) const;
	};

public:
	FOpenXRHandTracking(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);
	virtual ~FOpenXRHandTracking();

	/** IOpenXRExtensionPlugin */
	virtual FString GetDisplayName() override
	{
		return FString(TEXT("OpenXRHandTracking"));
	}
	virtual bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual const void* OnGetSystem(XrInstance InInstance, const void* InNext) override;
	virtual const void* OnCreateSession(XrInstance InInstance, XrSystemId InSystem, const void* InNext) override;
	virtual const void* OnBeginSession(XrSession InSession, const void* InNext) override;
	virtual void UpdateDeviceLocations(XrSession InSession, XrTime DisplayTime, XrSpace TrackingSpace) override;

	/** IMotionController interface */
	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const override;
	virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override;
	virtual FName GetMotionControllerDeviceTypeName() const override;
	virtual void EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const override;

	// ILiveLinkSource interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual bool IsSourceStillValid() const override;
	virtual bool RequestSourceShutdown() override;
	virtual FText GetSourceMachineName() const override;
	virtual FText GetSourceStatus() const override;
	virtual FText GetSourceType() const override;
	// End ILiveLinkSource

	/** IInputDevice interface */
	virtual void Tick(float DeltaTime) override;
	virtual void SendControllerEvents() override;
	virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override {};
	virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override {};
	virtual bool SupportsForceFeedback(int32 ControllerId) override { return false; }
	virtual bool IsGamepadAttached() const override;

	/** IHandTracker */
	virtual FName GetHandTrackerDeviceTypeName() const override;
	virtual bool IsHandTrackingStateValid() const override;
	virtual bool GetKeypointState(EControllerHand Hand, EHandKeypoint Keypoint, FTransform& OutTransform, float& OutRadius) const override;
	virtual bool GetAllKeypointStates(EControllerHand Hand, TArray<FVector>& OutPositions, TArray<FQuat>& OutRotations, TArray<float>& OutRadii, bool& OutIsTracked) const override;

private:
	FHandState& GetLeftHandState();
	FHandState& GetRightHandState();
public:
	const FHandState& GetLeftHandState() const;
	const FHandState& GetRightHandState() const;
	bool IsHandTrackingSupportedByDevice() const;

	/** Parses the enum name removing the prefix */
	static FName ParseEOpenXRHandKeypointEnumName(FName EnumName)
	{
		static int32 EnumNameLength = FString(TEXT("EHandKeypoint::")).Len();
		FString EnumString = EnumName.ToString();
		return FName(*EnumString.Right(EnumString.Len() - EnumNameLength));
	}

private:
	void AddKeys();
	void BuildMotionSourceToKeypointMap();
	
	void SetupLiveLinkData();
	void UpdateLiveLink();
	void UpdateLiveLinkTransforms(TArray<FTransform>& OutTransforms, const FOpenXRHandTracking::FHandState& HandState);
	void CreateHandTracker(XrSession InSession, XrHandEXT HandEXT, FHandState& HandState);

	bool bHandTrackingAvailable = false;

	PFN_xrCreateHandTrackerEXT xrCreateHandTrackerEXT = nullptr;
	PFN_xrDestroyHandTrackerEXT xrDestroyHandTrackerEXT = nullptr;
	PFN_xrLocateHandJointsEXT xrLocateHandJointsEXT = nullptr;

	class IXRTrackingSystem* XRTrackingSystem = nullptr;

	TSharedPtr<FGenericApplicationMessageHandler> MessageHandler;
	int32 DeviceIndex;

	int32 CurrentHandTrackingDataIndex = 0;

	TArray<int32> BoneParents;
	TArray<EHandKeypoint> BoneKeypoints;
	typedef TPair<EHandKeypoint, bool> MotionSourceInfo; // bool true == left, false == right
	TMap<FName, MotionSourceInfo> MotionSourceToKeypointMap;
	bool bSupportLegacyControllerMotionSources = true;

	FHandState HandStates[2];

	// LiveLink Data
	/** The local client to push data updates to */
	ILiveLinkClient* LiveLinkClient = nullptr;
	/** Our identifier in LiveLink */
	FGuid LiveLinkSourceGuid;

	static FLiveLinkSubjectName LiveLinkLeftHandTrackingSubjectName;
	static FLiveLinkSubjectName LiveLinkRightHandTrackingSubjectName;
	FLiveLinkSubjectKey LiveLinkLeftHandTrackingSubjectKey;
	FLiveLinkSubjectKey LiveLinkRightHandTrackingSubjectKey;
	bool bNewLiveLinkClient = false;
	FLiveLinkStaticDataStruct LiveLinkSkeletonStaticData;

	TArray<FTransform> LeftAnimationTransforms;
	TArray<FTransform> RightAnimationTransforms;
};

DEFINE_LOG_CATEGORY_STATIC(LogOpenXRHandTracking, Display, All);
