// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHandTracking.h"
#include "ILiveLinkClient.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"

#define LOCTEXT_NAMESPACE "OpenXRHandTracking"

void FOpenXRHandTracking::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	LiveLinkClient = InClient;
	LiveLinkSourceGuid = InSourceGuid;
	bNewLiveLinkClient = true;

	LiveLinkLeftHandTrackingSubjectKey.Source = InSourceGuid;
	LiveLinkLeftHandTrackingSubjectKey.SubjectName = LiveLinkLeftHandTrackingSubjectName;
	LiveLinkRightHandTrackingSubjectKey.Source = InSourceGuid;
	LiveLinkRightHandTrackingSubjectKey.SubjectName = LiveLinkRightHandTrackingSubjectName;

	UpdateLiveLink();
}

bool FOpenXRHandTracking::IsSourceStillValid() const
{
	return LiveLinkClient != nullptr;
}

bool FOpenXRHandTracking::RequestSourceShutdown()
{
	LiveLinkClient = nullptr;
	LiveLinkSourceGuid.Invalidate();
	return true;
}

FText FOpenXRHandTracking::GetSourceMachineName() const
{
	return FText().FromString(FPlatformProcess::ComputerName());
}

FText FOpenXRHandTracking::GetSourceStatus() const
{
	return LOCTEXT("OpenXRHandTrackingLiveLinkStatus", "Active");
}

FText FOpenXRHandTracking::GetSourceType() const
{
	return LOCTEXT("OpenXRHandTrackingLiveLinkSourceType", "OpenXR Hand Tracking");
}

void FOpenXRHandTracking::SetupLiveLinkData()
{
	check(IsInGameThread());

	LiveLinkSkeletonStaticData.InitializeWith(FLiveLinkSkeletonStaticData::StaticStruct(), nullptr);
	FLiveLinkSkeletonStaticData* SkeletonDataPtr = LiveLinkSkeletonStaticData.Cast<FLiveLinkSkeletonStaticData>();
	check(SkeletonDataPtr);

	TArray<FName>& BoneNames = SkeletonDataPtr->BoneNames;
	BoneNames.Reserve(EHandKeypointCount);
	// Array of bone indices to parent bone index
	BoneParents.Reserve(EHandKeypointCount);
	BoneKeypoints.Reserve(EHandKeypointCount);

	const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/HeadMountedDisplay.EHandKeypoint"), EFindObjectFlags::ExactClass);
	check(EnumPtr != nullptr);
	// Iterate through all of the Keypoints building the skeleton info for it
	for (int32 Keypoint = 0; Keypoint < EHandKeypointCount; Keypoint++)
	{
		BoneKeypoints.Add((EHandKeypoint)Keypoint);
		BoneNames.Add(FOpenXRHandTracking::ParseEOpenXRHandKeypointEnumName(EnumPtr->GetNameByValue(Keypoint)));
	}

	// Manually build the parent hierarchy starting at the wrist which has no parent (-1)
	BoneParents.Add(1);		// Palm
	BoneParents.Add(-1);	// Wrist -> Palm

	BoneParents.Add(1);		// ThumbMetacarpal -> Wrist
	BoneParents.Add(2);		// ThumbProximal -> ThumbMetacarpal
	BoneParents.Add(3);		// ThumbDistal -> ThumbProximal
	BoneParents.Add(4);		// ThumbTip -> ThumbDistal

	BoneParents.Add(1);		// IndexMetacarpal -> Wrist
	BoneParents.Add(6);		// IndexProximal -> IndexMetacarpal
	BoneParents.Add(7);		// IndexIntermediate -> IndexProximal
	BoneParents.Add(8);		// IndexDistal -> IndexIntermediate
	BoneParents.Add(9);		// IndexTip -> IndexDistal

	BoneParents.Add(1);		// MiddleMetacarpal -> Wrist
	BoneParents.Add(11);	// MiddleProximal -> MiddleMetacarpal
	BoneParents.Add(12);	// MiddleIntermediate -> MiddleProximal
	BoneParents.Add(13);	// MiddleDistal -> MiddleIntermediate
	BoneParents.Add(14);	// MiddleTip -> MiddleDistal

	BoneParents.Add(1);		// RingMetacarpal -> Wrist
	BoneParents.Add(16);	// RingProximal -> RingMetacarpal
	BoneParents.Add(17);	// RingIntermediate -> RingProximal
	BoneParents.Add(18);	// RingDistal -> RingIntermediate
	BoneParents.Add(19);	// RingTip -> RingDistal

	BoneParents.Add(1);		// LittleMetacarpal -> Wrist
	BoneParents.Add(21);	// LittleProximal -> LittleMetacarpal
	BoneParents.Add(22);	// LittleIntermediate -> LittleProximal
	BoneParents.Add(23);	// LittleDistal -> LittleIntermediate
	BoneParents.Add(24);	// LittleTip -> LittleDistal

	SkeletonDataPtr->SetBoneParents(BoneParents);
}

void FOpenXRHandTracking::UpdateLiveLinkTransforms(TArray<FTransform>& OutTransforms, const FOpenXRHandTracking::FHandState& HandState)
{
	// Live link transforms need to be in the hierarchical skeleton, so each in the space of its parent.
	// The hand tracking transforms are in world space.
	for (int32 Index = 0; Index < EHandKeypointCount; ++Index)
	{
		const FTransform& BoneTransform = HandState.GetTransform(BoneKeypoints[Index]);
		int32 ParentIndex = BoneParents[Index];
		if (ParentIndex < 0)
		{
			// We are at the root, so use it.
			OutTransforms[Index] = BoneTransform;
		}
		else
		{
			const FTransform& ParentTransform = HandState.GetTransform(BoneKeypoints[ParentIndex]);
			OutTransforms[Index] = BoneTransform * ParentTransform.Inverse();
		}
	}
}

void FOpenXRHandTracking::UpdateLiveLink()
{
	check(IsInGameThread());

	if (LiveLinkClient)
	{
		// One time initialization:
		if (LeftAnimationTransforms.Num() == 0)
		{
			check(EHandKeypointCount > 0); // ensure the num() test above is a valid way to detect initialization

			SetupLiveLinkData();

			LeftAnimationTransforms.Reserve(EHandKeypointCount);
			RightAnimationTransforms.Reserve(EHandKeypointCount);
			// Init to identity all of the Keypoint transforms
			for (uint32 Count = 0; Count < EHandKeypointCount; ++Count)
			{
				LeftAnimationTransforms.Add(FTransform::Identity);
				RightAnimationTransforms.Add(FTransform::Identity);
			}
		}

		// Per ReceiveClient initialization:
		if (bNewLiveLinkClient)
		{
			FLiveLinkStaticDataStruct SkeletalDataLeft;
			SkeletalDataLeft.InitializeWith(LiveLinkSkeletonStaticData);
			FLiveLinkStaticDataStruct SkeletalDataRight;
			SkeletalDataRight.InitializeWith(LiveLinkSkeletonStaticData);

			LiveLinkClient->RemoveSubject_AnyThread(LiveLinkLeftHandTrackingSubjectKey);
			LiveLinkClient->RemoveSubject_AnyThread(LiveLinkRightHandTrackingSubjectKey);
			LiveLinkClient->PushSubjectStaticData_AnyThread(LiveLinkLeftHandTrackingSubjectKey, ULiveLinkAnimationRole::StaticClass(), MoveTemp(SkeletalDataLeft));
			LiveLinkClient->PushSubjectStaticData_AnyThread(LiveLinkRightHandTrackingSubjectKey, ULiveLinkAnimationRole::StaticClass(), MoveTemp(SkeletalDataRight));
			bNewLiveLinkClient = false;
		}

		// Every frame updates:

		// Update the transforms for each subject from tracking data
		UpdateLiveLinkTransforms(LeftAnimationTransforms, GetLeftHandState());
		UpdateLiveLinkTransforms(RightAnimationTransforms, GetRightHandState());

		{
			// Note these structures will be Moved to live link, leaving them invalid.
			FLiveLinkFrameDataStruct LiveLinkLeftFrameDataStruct(FLiveLinkAnimationFrameData::StaticStruct());
			FLiveLinkAnimationFrameData& LiveLinkLeftAnimationFrameData = *LiveLinkLeftFrameDataStruct.Cast<FLiveLinkAnimationFrameData>();
			FLiveLinkFrameDataStruct LiveLinkRightFrameDataStruct(FLiveLinkAnimationFrameData::StaticStruct());
			FLiveLinkAnimationFrameData& LiveLinkRightAnimationFrameData = *LiveLinkRightFrameDataStruct.Cast<FLiveLinkAnimationFrameData>();
			static FName HandednessName (TEXT("Handedness"));
			LiveLinkLeftAnimationFrameData.MetaData.StringMetaData.Add(HandednessName, TEXT("Left"));
			LiveLinkRightAnimationFrameData.MetaData.StringMetaData.Add(HandednessName, TEXT("Right"));
			LiveLinkLeftAnimationFrameData.WorldTime = LiveLinkRightAnimationFrameData.WorldTime = FPlatformTime::Seconds();

			//Copy transforms over to transient structure
			LiveLinkLeftAnimationFrameData.Transforms = LeftAnimationTransforms;
			LiveLinkRightAnimationFrameData.Transforms = RightAnimationTransforms;

			// Share the data locally with the LiveLink client
			LiveLinkClient->PushSubjectFrameData_AnyThread(LiveLinkLeftHandTrackingSubjectKey, MoveTemp(LiveLinkLeftFrameDataStruct));
			LiveLinkClient->PushSubjectFrameData_AnyThread(LiveLinkRightHandTrackingSubjectKey, MoveTemp(LiveLinkRightFrameDataStruct));
		}
	}
}

#undef LOCTEXT_NAMESPACE
