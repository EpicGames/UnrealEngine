// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHandTrackingLiveLinkRemapAsset.h"
#include "OpenXRHandTracking.h"

#include "BonePose.h"
#include "Roles/LiveLinkAnimationTypes.h"

#define LOCTEXT_NAMESPACE "OpenXRHandTrackingLiveLinkRemapAsset"

UOpenXRHandTrackingLiveLinkRemapAsset::UOpenXRHandTrackingLiveLinkRemapAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITORONLY_DATA
void UOpenXRHandTrackingLiveLinkRemapAsset::PostInitProperties()
{
	Super::PostInitProperties();

	if (HandTrackingBoneNameMap.Num() == 0)
	{
		// Fill the first time use of the bone name map
		const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/HeadMountedDisplay.EHandKeypoint"), EFindObjectFlags::ExactClass);
		check(EnumPtr != nullptr);
		// Iterate through all of the Keypoints building the skeleton info for it
		for (int32 Keypoint = 0; Keypoint < EHandKeypointCount; Keypoint++)
		{
			FName BoneName = FOpenXRHandTracking::ParseEOpenXRHandKeypointEnumName(EnumPtr->GetNameByValue(Keypoint));
			HandTrackingBoneNameMap.Add(BoneName, BoneName);
		}
	}
}
#endif

void UOpenXRHandTrackingLiveLinkRemapAsset::BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose)
{
	check(InSkeletonData);
	check(InFrameData);

	// Transform Bone Names
	const TArray<FName>& SourceBoneNames = InSkeletonData->GetBoneNames();

	TArray<FName, TMemStackAllocator<>> TransformedBoneNames;
	TransformedBoneNames.Reserve(SourceBoneNames.Num());

	for (const FName& SrcBoneName : SourceBoneNames)
	{
		TransformedBoneNames.Add(GetRemappedBoneName(SrcBoneName));
	}

	for (int32 i = 0; i < TransformedBoneNames.Num(); ++i)
	{
		FName BoneName = TransformedBoneNames[i];

		FTransform BoneTransform = GetRetargetedTransform(InFrameData, i);

		int32 MeshIndex = OutPose.GetBoneContainer().GetPoseBoneIndexForBoneName(BoneName);
		if (MeshIndex != INDEX_NONE)
		{
			FCompactPoseBoneIndex CPIndex = OutPose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshIndex));
			if (CPIndex != INDEX_NONE)
			{
				FQuat OldRot = BoneTransform.GetRotation();
				FQuat NewRot;
				FVector4 OldRotVector(OldRot.X, OldRot.Y, OldRot.Z, OldRot.W);

				NewRot.X = Sign(SwizzleX) * OldRotVector[StaticCast<uint8>(SwizzleX) % 4];
				NewRot.Y = Sign(SwizzleY) * OldRotVector[StaticCast<uint8>(SwizzleY) % 4];
				NewRot.Z = Sign(SwizzleZ) * OldRotVector[StaticCast<uint8>(SwizzleZ) % 4];
				NewRot.W = Sign(SwizzleW) * OldRotVector[StaticCast<uint8>(SwizzleW) % 4];

				// Left hand bones need to be flipped
				const FString* Handedness = InFrameData->MetaData.StringMetaData.Find(TEXT("Handedness"));
				if (Handedness && Handedness->Equals(TEXT("Left")))
				{
					NewRot.X = -NewRot.X;
					NewRot.Y = -NewRot.Y;
				}

				NewRot.Normalize();

				if (bRetargetRotationOnly)
				{
					// Ref pose rotated by the live link transform
					OutPose[CPIndex] = OutPose.GetRefPose(CPIndex);
				}
				else
				{
					OutPose[CPIndex] = BoneTransform;
				}

				OutPose[CPIndex].SetRotation(NewRot);
			}
		}
	}
}

FName UOpenXRHandTrackingLiveLinkRemapAsset::GetRemappedBoneName(FName BoneName) const
{
	// Return the mapped name if we have one, otherwise just pass back the base name
	const FName* OutName = HandTrackingBoneNameMap.Find(BoneName);
	if (OutName != nullptr)
	{
		return *OutName;
	}
	return BoneName;
}

FTransform UOpenXRHandTrackingLiveLinkRemapAsset::GetRetargetedTransform(const FLiveLinkAnimationFrameData* InFrameData, int TransformIndex) const
{
	check(InFrameData);

	FTransform OutTransform = InFrameData->Transforms[TransformIndex];

	if (!bHasMetacarpals && (
		TransformIndex == static_cast<int>(EHandKeypoint::ThumbProximal)		|| 
		TransformIndex == static_cast<int>(EHandKeypoint::IndexProximal)		||
		TransformIndex == static_cast<int>(EHandKeypoint::MiddleProximal)	|| 
		TransformIndex == static_cast<int>(EHandKeypoint::RingProximal)		|| 
		TransformIndex == static_cast<int>(EHandKeypoint::LittleProximal)	))
	{
		// Metacarpal is always one entry before the Proximal
		OutTransform = InFrameData->Transforms[TransformIndex - 1] * OutTransform;
	}

	return OutTransform;
}

#undef LOCTEXT_NAMESPACE

int Sign(const EQuatSwizzleAxisB& QuatSwizzleAxis)
{
	return (QuatSwizzleAxis > EQuatSwizzleAxisB::W) ? -1 : 1;
}
