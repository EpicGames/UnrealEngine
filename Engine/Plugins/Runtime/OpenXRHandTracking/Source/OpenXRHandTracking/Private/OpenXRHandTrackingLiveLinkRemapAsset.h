// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Package.h"
#include "LiveLinkRetargetAsset.h"
#include "OpenXRHandTrackingLiveLinkRemapAsset.generated.h"

UENUM()
enum class EQuatSwizzleAxisB : uint8
{
	X		UMETA(DisplayName = "X"),
	Y		UMETA(DisplayName = "Y"),
	Z		UMETA(DisplayName = "Z"),
	W		UMETA(DisplayName = "W"),
	MinusX	UMETA(DisplayName = "-X"),
	MinusY	UMETA(DisplayName = "-Y"),
	MinusZ	UMETA(DisplayName = "-Z"),
	MinusW	UMETA(DisplayName = "-W")
};

int Sign(const EQuatSwizzleAxisB& QuatSwizzleAxis);

/**
  * OpenXR HandTracking LiveLink remapping asset
  */
UCLASS(Blueprintable)
class UOpenXRHandTrackingLiveLinkRemapAsset :
	public ULiveLinkRetargetAsset
{
	GENERATED_UCLASS_BODY()

public:
	virtual void BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) override;

	/** If true, remap the full human hand skeleton including metacarpals */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	bool bHasMetacarpals = true;

	/** Only apply the orientation to each bone */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	bool bRetargetRotationOnly = false;

	/** Reorient the skeleton (swizzle the quaternion) to adjust for base skeleton and incoming skeleton differences */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EQuatSwizzleAxisB SwizzleX = EQuatSwizzleAxisB::X;
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EQuatSwizzleAxisB SwizzleY = EQuatSwizzleAxisB::Y;
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EQuatSwizzleAxisB SwizzleZ = EQuatSwizzleAxisB::Z;
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EQuatSwizzleAxisB SwizzleW = EQuatSwizzleAxisB::W;

	UPROPERTY(EditAnywhere, Category="LiveLink")
	TMap<FName, FName> HandTrackingBoneNameMap;

#if WITH_EDITORONLY_DATA
	virtual void PostInitProperties() override;
#endif

private:
	FName GetRemappedBoneName(FName BoneName) const;

	FTransform GetRetargetedTransform(const FLiveLinkAnimationFrameData* InFrameData, int TransformIndex) const;
};
