// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHandTrackingSettings.h"
//#include "XRMotionControllerBase.h"
//#include "PlayerMappableInputConfig.h"

//#if WITH_EDITOR
//#include "Editor.h"
//#include "Features/IModularFeatures.h"
//#endif

UOpenXRHandTrackingSettings::UOpenXRHandTrackingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
