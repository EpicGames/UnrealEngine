// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IInputDeviceModule.h"

enum class EHandKeypoint : uint8;
enum class EControllerHand : uint8;

/**
 * The public interface to this module.  In most cases, this interface is only public to sibling modules
 * within this plugin.
 */
class IOpenXRHandTrackingModule : public IInputDeviceModule
{
public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though. Your module might have been
	* unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IOpenXRHandTrackingModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IOpenXRHandTrackingModule>("OpenXRHandTracking");
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if
	* IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("OpenXRHandTracking");
	}

	/**
	 * Returns the Input Device associated with this InputDeviceModule.
	 *
	 * @return Shared pointer to the Input Device.
	 */
	virtual TSharedPtr<IInputDevice> GetInputDevice() = 0;

	/**
	* Returns the LiveLinkSource associated with this OpenXRHandTrackingPlugin.
	*
	* @return Shared pointer to the Input Device.
	*/
	virtual TSharedPtr<class ILiveLinkSource> GetLiveLinkSource() = 0;

	/**
	 * Checks if the LiveLinkSource as been created.
	 *
	 * @return True if the LiveLinkSource has been created with GetLiveLinkSource or AddLiveLinkSource
	 */
	virtual bool IsLiveLinkSourceValid() const = 0;

	virtual void AddLiveLinkSource() = 0;
	virtual void RemoveLiveLinkSource() = 0;
};
