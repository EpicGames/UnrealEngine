// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "OpenXRHandTrackingSettings.generated.h"

/**
* Implements the settings for the OpenXR Input plugin.
*/
UCLASS(config = Input, defaultconfig)
class OPENXRHANDTRACKING_API UOpenXRHandTrackingSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** If false (the default) the motion sources for hand tracking will be of the form '[Left|Right][Keypoint]'.  If true they will be of the form 'HandTracking[Left|Right][Keypoint]'.  True is reccomended to avoid collisions between motion sources from different device types. **/
	UPROPERTY(config, EditAnywhere, Category = "Motion Sources")
	bool bUseMoreSpecificMotionSourceNames = false;

	/** If true hand tracking supports the 'Left' and 'Right' legacy motion sources.  If false it does not.  False is reccomended unless you need legacy compatibility in an older unreal projects.**/
	UPROPERTY(config, EditAnywhere, Category = "Motion Sources")
	bool bSupportLegacyControllerMotionSources = true;
};
