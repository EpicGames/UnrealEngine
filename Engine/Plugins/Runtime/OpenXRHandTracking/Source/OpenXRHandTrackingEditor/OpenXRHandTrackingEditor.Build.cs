// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OpenXRHandTrackingEditor : ModuleRules
{
	public OpenXRHandTrackingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"EditorFramework",
				"UnrealEd",
				"OpenXRHandTracking"
			}
		);
	}
}
