// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHandTrackingSettings.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"


#define LOCTEXT_NAMESPACE "FOpenXRHandTrackingEditorModule"

/**
 * Module for the editor module.
 */
class FOpenXRHandTrackingEditorModule
	: public IModuleInterface
{
public:

	/** Default constructor. */
	FOpenXRHandTrackingEditorModule( )
	{ }

	/** Destructor. */
	~FOpenXRHandTrackingEditorModule( )
	{
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "OpenXRHandTracking",
				LOCTEXT("SettingsName", "OpenXRHandTracking"),
				LOCTEXT("SettingsDescription", "Settings for OpenXRHandTracking Plugin"),
				GetMutableDefault<UOpenXRHandTrackingSettings>()
			);
		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "OpenXRHandTracking");
		}
	}

private:

};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FOpenXRHandTrackingEditorModule, OpenXRHandTrackingEditor);

